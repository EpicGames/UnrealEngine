{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ReplayKit for iOS",
	"Description": "Support for local recording and broadcasting using ReplayKit",
	"Category": "Mobile",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "IOSReplayKit",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false
}