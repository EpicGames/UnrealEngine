// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class IOSReplayKit : ModuleRules
	{
		public IOSReplayKit(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;

			PrivateDependencyModuleNames.AddRange(new string[]
				{
					"Core",
					"CoreUObject",
					"Engine"
				});

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}

			if (Target.Platform == UnrealTargetPlatform.IOS)
			{
				PublicFrameworks.AddRange( new string[]{
					"ReplayKit"
				});

                string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
                AdditionalPropertiesForReceipt.Add("IOSPlugin", Path.Combine(PluginPath, "IOSReplayKit_UPL.xml"));
			}
		}
	}
}
