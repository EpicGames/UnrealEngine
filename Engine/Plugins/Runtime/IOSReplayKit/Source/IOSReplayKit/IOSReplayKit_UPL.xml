<?xml version="1.0" encoding="utf-8"?>
<root>
    <init>
        <log text="IOSReplayKit adding camera roll permission to plist..."/>
    </init>
    <iosPListUpdates>
        <addElements tag="dict" once="true">
            <key>NSPhotoLibraryAddUsageDescription</key>
            <string>This app saves screen captures to the photo library.</string>
        </addElements>
    </iosPListUpdates>
</root>
