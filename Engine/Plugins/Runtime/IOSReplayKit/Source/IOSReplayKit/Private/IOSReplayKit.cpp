// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSReplayKit.h"

DEFINE_LOG_CATEGORY( LogIOSReplayKit );

#if PLATFORM_IOS

#include "IOSAppDelegate.h"
#import "ReplayKitRecorder.h"
#include "IOSReplayKitControl.h"

class FIOSReplayKit : public IIOSReplayKitModuleInterface, public FSelfRegisteringExec
{
public:
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;

	virtual void Initialize( bool bMicrophoneEnabled = false, bool bCameraEnabled = false ) override;
	
	virtual void StartRecording() override;
	virtual void StopRecording() override;
    
    virtual void StartCaptureToFile() override;
    virtual void StopCapture() override;
	
	virtual void StartBroadcast() override;
	virtual void PauseBroadcast() override;
	virtual void ResumeBroadcast() override;
	virtual void StopBroadcast() override;
	
private:
	ReplayKitRecorder* rkr;
};

bool FIOSReplayKit::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("RKStart")))
	{
		UIOSReplayKitControl::StartRecording();
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("RKStop")))
	{
		UIOSReplayKitControl::StopRecording();
		return true;
	}

	return false;
}


void FIOSReplayKit::Initialize( bool bMicrophoneEnabled, bool bCameraEnabled )
{
	BOOL _bMicrophoneEnabled = bMicrophoneEnabled ? YES : NO;
	// camera capture is disabled because it requires a UIView to display
	BOOL _bCameraEnabled = NO;
	
	rkr = [[ReplayKitRecorder alloc] init];
	[rkr initializeWithMicrophoneEnabled:_bMicrophoneEnabled withCameraEnabled:_bCameraEnabled];

}

void FIOSReplayKit::StartRecording()
{
	[rkr startRecording];
}

void FIOSReplayKit::StopRecording()
{
	[rkr stopRecording];
}

void FIOSReplayKit::StartCaptureToFile()
{
    [rkr startCapture];
}

void FIOSReplayKit::StopCapture()
{
    [rkr stopCapture];
}

void FIOSReplayKit::StartBroadcast()
{
	[rkr startBroadcast];
}

void FIOSReplayKit::PauseBroadcast()
{
	[rkr pauseBroadcast];
}

void FIOSReplayKit::ResumeBroadcast()
{
	[rkr resumeBroadcast];
}

void FIOSReplayKit::StopBroadcast()
{
	[rkr stopBroadcast];
}

#else 

class FIOSReplayKit : public IIOSReplayKitModuleInterface
{
public:
    
    virtual void Initialize( bool bMicrophoneEnabled = false, bool bCameraEnabled = false )
    {
        UE_LOGF(LogIOSReplayKit, Display, "ReplayKit not available on this platform");
    }
    
    virtual void StartRecording() override {}
    virtual void StopRecording() override {}
    
    virtual void StartCaptureToFile() override {}
    virtual void StopCapture() override {}
    
    virtual void StartBroadcast() override {}
    virtual void PauseBroadcast() override {}
    virtual void ResumeBroadcast() override {}
    virtual void StopBroadcast() override {}
};

#endif

void IIOSReplayKitModuleInterface::StartupModule()
{
    // This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
}


void IIOSReplayKitModuleInterface::ShutdownModule()
{
    // This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
    // we call this function before unloading the module.
}

IMPLEMENT_MODULE(FIOSReplayKit, IOSReplayKit)



