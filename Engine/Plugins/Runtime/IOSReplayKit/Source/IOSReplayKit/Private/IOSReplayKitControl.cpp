// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSReplayKitControl.h"
#include "IOSReplayKit.h"

void UIOSReplayKitControl::StartRecording(bool bMicrophoneEnabled)
{
    IIOSReplayKitModuleInterface::Get().Initialize(bMicrophoneEnabled, false);
    IIOSReplayKitModuleInterface::Get().StartRecording();
	
    UE_LOGF(LogIOSReplayKit, Log, "Starting recording!");
}

void UIOSReplayKitControl::StopRecording()
{
    IIOSReplayKitModuleInterface::Get().StopRecording();
    UE_LOGF(LogIOSReplayKit, Log, "Stopping recording!");
}

void UIOSReplayKitControl::StartCaptureToFile(bool bMicrophoneEnabled)
{
    IIOSReplayKitModuleInterface::Get().Initialize(bMicrophoneEnabled, false);
    IIOSReplayKitModuleInterface::Get().StartCaptureToFile();
}

void UIOSReplayKitControl::StopCapture()
{
    IIOSReplayKitModuleInterface::Get().StopCapture();
}
