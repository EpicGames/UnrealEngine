// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReplayKitRecorder.h"



#if PLATFORM_IOS
#include "IOSAppDelegate.h"
#include "IOS/IOSView.h"
#include "Misc/Paths.h"
#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "UnrealClient.h"

@implementation ReplayKitRecorder

RPScreenRecorder* _Nullable _screenRecorder;
RPBroadcastActivityViewController* _Nullable _broadcastActivityController;
RPBroadcastController* _Nullable _broadcastController;

// stuff used when capturing to file
AVAssetWriter* _Nullable _assetWriter;
AVAssetWriterInput* _Nullable _videoInput;
AVAssetWriterInput* _Nullable _audioInput;
NSString* _Nullable _captureFilePath;

- (void) initializeWithMicrophoneEnabled:(BOOL)bMicrophoneEnabled withCameraEnabled:(BOOL)bCameraEnabled {
	_screenRecorder = [RPScreenRecorder sharedRecorder];
	[_screenRecorder setDelegate:self];
#if !PLATFORM_TVOS
	[_screenRecorder setMicrophoneEnabled:bMicrophoneEnabled];
	[_screenRecorder setCameraEnabled:bCameraEnabled];
#endif
}

- (void)startRecording {
	// NOTE(omar): stop any live broadcasts before staring a local recording
	if( _broadcastController != nil ) {
		[self stopBroadcast];
	}
	
	if( [_screenRecorder isAvailable] ) {
		[_screenRecorder startRecordingWithHandler:^(NSError * _Nullable error) {
			if( error ) {
				NSLog( @"error starting screen recording");
			}
		}];
	}
}

- (void)stopRecording {
	if( [_screenRecorder isAvailable] && [_screenRecorder isRecording] ) {
		[_screenRecorder stopRecordingWithHandler:^(RPPreviewViewController * _Nullable previewViewController, NSError * _Nullable error) {
			[previewViewController setPreviewControllerDelegate:self];
		 
			// automatically show the video preview when recording is stopped
			previewViewController.popoverPresentationController.sourceView = (UIView* _Nullable)[IOSAppDelegate GetDelegate].IOSView;
			[[IOSAppDelegate GetDelegate].IOSController presentViewController:previewViewController animated:YES completion:nil];
		}];
	}
}

- (void)createCaptureContext
{
    auto fileManager = [NSFileManager defaultManager];
    auto docDir = [NSSearchPathForDirectoriesInDomains(NSDocumentDirectory, NSUserDomainMask, YES) objectAtIndex:0];
    auto captureDir = [docDir stringByAppendingFormat:@"/Captures"];
    [fileManager createDirectoryAtPath:captureDir withIntermediateDirectories:YES attributes:nil error:nil];
    
    [_captureFilePath release];
    
    // create the asset writer
    [_assetWriter release];
    // todo: do we care about the file name? support cleaning up old captures?
    auto fileName = FString::Printf(TEXT("%s.mp4"), *FGuid::NewGuid().ToString());;
    _captureFilePath = [captureDir stringByAppendingFormat:@"/%@", fileName.GetNSString()];
    [_captureFilePath retain];
    
    _assetWriter = [AVAssetWriter assetWriterWithURL:[NSURL fileURLWithPath:_captureFilePath] fileType:AVFileTypeMPEG4 error:nil];
    [_assetWriter retain];
    
    // create the video input
    [_videoInput release];
    
    auto view = [IOSAppDelegate GetDelegate].IOSView;
    auto width= [NSNumber numberWithFloat:view.frame.size.width];
    auto height = [NSNumber numberWithFloat:view.frame.size.height];
    
    if (GEngine && GEngine->GameViewport && GEngine->GameViewport->Viewport)
    {
        auto viewportSize = GEngine->GameViewport->Viewport->GetSizeXY();
        width = [NSNumber numberWithInt:viewportSize.X];
        height = [NSNumber numberWithInt:viewportSize.Y];
    }
    
    auto videoSettings = @{
                           AVVideoCodecKey  : AVVideoCodecTypeH264,
                           AVVideoWidthKey  : width,
                           AVVideoHeightKey : height
                           };
    
    _videoInput = [AVAssetWriterInput assetWriterInputWithMediaType:AVMediaTypeVideo outputSettings:videoSettings];
    _videoInput.expectsMediaDataInRealTime = YES;
    [_videoInput retain];
    
    // create the audio input
    [_audioInput release];
    AudioChannelLayout acl;
    bzero(&acl, sizeof(acl));
    acl.mChannelLayoutTag = kAudioChannelLayoutTag_Stereo;
    auto audioSettings = @{
                           AVFormatIDKey : @(kAudioFormatMPEG4AAC),
                           AVSampleRateKey: @(44100),
                           AVChannelLayoutKey: [NSData dataWithBytes:&acl length:sizeof(acl)],
                           };
    _audioInput = [AVAssetWriterInput assetWriterInputWithMediaType:AVMediaTypeAudio outputSettings:audioSettings];
    _audioInput.expectsMediaDataInRealTime = YES;
    [_audioInput retain];
    
    // add the input to the writer
    if ([_assetWriter canAddInput:_videoInput])
    {
        [_assetWriter addInput:_videoInput];
    }
    
    if ([_assetWriter canAddInput:_audioInput])
    {
        [_assetWriter addInput:_audioInput];
    }
}

- (void)startCapture
{
    if (_broadcastController)
    {
        [self stopBroadcast];
    }
    
    if ([_screenRecorder isAvailable])
    {
        [self createCaptureContext];
        
        [_screenRecorder startCaptureWithHandler:^(CMSampleBufferRef sampleBuffer, RPSampleBufferType bufferType, NSError *error)
        {
            if (CMSampleBufferDataIsReady(sampleBuffer))
            {
                if (_assetWriter.status == AVAssetWriterStatusUnknown)
                {
                    [_assetWriter startWriting];
                    [_assetWriter startSessionAtSourceTime:CMSampleBufferGetPresentationTimeStamp(sampleBuffer)];
                }
                
                if (_assetWriter.status == AVAssetWriterStatusFailed)
                {
                    NSLog(@"%d: %@", (int)_assetWriter.error.code, _assetWriter.error.localizedDescription);
                    return;
                }
                
                AVAssetWriterInput* input = nullptr;
                
                if (bufferType == RPSampleBufferTypeVideo)
                {
                    input = _videoInput;
                }
                else if (bufferType == RPSampleBufferTypeAudioApp)
                {
                    input = _audioInput;
                }
                else if (bufferType == RPSampleBufferTypeAudioMic)
                {
                    // todo?
                }
                
                if (input && input.isReadyForMoreMediaData)
                {
                    [input appendSampleBuffer:sampleBuffer];
                }
            }
        }
         
        completionHandler:^(NSError *error)
        {
            if (error)
            {
                NSLog(@"completionHandler: %@", error);
            }
        }];
    }
}

- (void)stopCapture
{
    if ([_screenRecorder isAvailable])
    {
        [_screenRecorder stopCaptureWithHandler:^(NSError *error)
        {
            if (error)
            {
                NSLog(@"stopCaptureWithHandler: %@", error);
            }
            
            if (_assetWriter)
            {
                [_assetWriter finishWritingWithCompletionHandler:^()
                {
                    NSLog(@"finishWritingWithCompletionHandler");
                    
#if !PLATFORM_TVOS
                    if (UIVideoAtPathIsCompatibleWithSavedPhotosAlbum(_captureFilePath))
                    {
                        UISaveVideoAtPathToSavedPhotosAlbum(_captureFilePath, nil, nil, nil);
                        NSLog(@"capture saved to album");
                    }
#endif

                    [_captureFilePath release];
                    _captureFilePath = nullptr;
                    
                    [_videoInput release];
                    _videoInput = nullptr;
                    
                    [_audioInput release];
                    _audioInput = nullptr;
                    
                    [_assetWriter release];
                    _assetWriter = nullptr;
                }];
            }
        }];
    }
}

//
// livestreaming functionality
//

- (void)startBroadcast {
	// NOTE(omar): ending any local recordings that might be active before starting a broadcast
	if( [_screenRecorder isRecording] ) {
		[self stopRecording];
	}
	
	[RPBroadcastActivityViewController loadBroadcastActivityViewControllerWithHandler:^(RPBroadcastActivityViewController * _Nullable broadcastActivityViewController, NSError * _Nullable error) {
		_broadcastActivityController = broadcastActivityViewController;
		[_broadcastActivityController setDelegate:self];
		[[IOSAppDelegate GetDelegate].IOSController presentViewController:_broadcastActivityController animated:YES completion:nil];
	}];
}

- (void)pauseBroadcast {
	if( [_broadcastController isBroadcasting] ) {
		[_broadcastController pauseBroadcast];
	}
}

- (void)resumeBroadcast {
	if( [_broadcastController isPaused] ) {
		[_broadcastController resumeBroadcast];
	}
}

- (void)stopBroadcast {
	if( [_broadcastController isBroadcasting ] ) {
		[_broadcastController finishBroadcastWithHandler:^(NSError * _Nullable error) {
			if( error ) {
				NSLog( @"error finishing broadcast" );
			}
			
			[_broadcastController release];
			_broadcastController = nil;
		}];
	}
}

//
// delegates
//

// screen recorder delegate
- (void)screenRecorder:(RPScreenRecorder* _Nullable)screenRecorder didStopRecordingWithError:(NSError* _Nullable)error previewViewController:(RPPreviewViewController* _Nullable)previewViewController {
	NSLog(@"RTRScreenRecorderDelegate::didStopRecrodingWithError");
	[previewViewController dismissViewControllerAnimated:YES completion:nil];
}

- (void)screenRecorderDidChangeAvailability:(RPScreenRecorder* _Nullable)screenRecorder {
	NSLog(@"RTRScreenRecorderDelegate::screenRecorderDidChangeAvailability");
}

// screen recorder preview view controller delegate
- (void)previewControllerDidFinish:(RPPreviewViewController* _Nullable)previewController {
	NSLog( @"RTRPreviewViewControllerDelegate::previewControllerDidFinish" );
	[previewController dismissViewControllerAnimated:YES completion:nil];
}

- (void)previewController:(RPPreviewViewController* _Nullable)previewController didFinishWithActivityTypes:(NSSet <NSString*> * _Nullable)activityTypes __TVOS_PROHIBITED {
	NSLog( @"RTRPreviewViewControllerDelegate::didFinishWithActivityTypes" );
	[previewController dismissViewControllerAnimated:YES completion:nil];
}

// broadcast activity view controller delegate
- (void)broadcastActivityViewController:(RPBroadcastActivityViewController* _Nullable)broadcastActivityViewController didFinishWithBroadcastController:(RPBroadcastController* _Nullable)broadcastController error:(NSError* _Nullable)error {
	NSLog( @"RPBroadcastActivityViewControllerDelegate::didFinishWithBroadcastController" );
	
	[broadcastActivityViewController dismissViewControllerAnimated:YES completion:^{
		_broadcastController = [broadcastController retain];
		[_broadcastController setDelegate:self];
		[_broadcastController startBroadcastWithHandler:^(NSError* _Nullable _error) {
			if( _error ) {
				NSLog( @"error starting broadcast" );
			}
		}];
	}];
}

// broadcast controller delegate
- (void)broadcastController:(RPBroadcastController* _Nullable)broadcastController didFinishWithError:(NSError* _Nullable)error {
	NSLog( @"RPBroadcastControllerDelegate::didFinishWithError" );
}

- (void)broadcastController:(RPBroadcastController* _Nullable)broadcastController didUpdateServiceInfo:(NSDictionary <NSString*, NSObject <NSCoding>*> *_Nullable)serviceInfo {
	NSLog( @"RPBroadcastControllerDelegate::didUpdateServiceInfo" );
}


@end

#endif
