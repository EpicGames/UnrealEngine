// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

#if PLATFORM_IOS

#import <Foundation/Foundation.h>
#import <ReplayKit/ReplayKit.h>

// screen recorder wrapper class
@interface ReplayKitRecorder : NSObject<RPScreenRecorderDelegate,
										RPPreviewViewControllerDelegate,
										RPBroadcastActivityViewControllerDelegate,
										RPBroadcastControllerDelegate>

- (void) initializeWithMicrophoneEnabled:(BOOL)bMicrophoneEnabled withCameraEnabled:(BOOL)bCameraEnabled;

// local recording
- (void)startRecording;
- (void)stopRecording;

- (void)startCapture;
- (void)stopCapture;

//
// livestreaming functionality
//`
- (void)startBroadcast;
- (void)pauseBroadcast;
- (void)resumeBroadcast;
- (void)stopBroadcast;

//
// delegates
//

// screen recorder delegate
- (void)screenRecorder:(RPScreenRecorder *_Nullable)screenRecorder didStopRecordingWithError:(NSError *_Nullable)error previewViewController:(nullable RPPreviewViewController *)previewViewController;
- (void)screenRecorderDidChangeAvailability:(RPScreenRecorder *_Nullable)screenRecorder;

// screen recorder preview view controller delegate
- (void)previewControllerDidFinish:(RPPreviewViewController *_Nullable)previewController;
- (void)previewController:(RPPreviewViewController *_Nullable)previewController didFinishWithActivityTypes:(NSSet <NSString *> *_Nullable)activityTypes __TVOS_PROHIBITED;

// broadcast activity view controller delegate
- (void)broadcastActivityViewController:(RPBroadcastActivityViewController *_Nullable)broadcastActivityViewController didFinishWithBroadcastController:(nullable RPBroadcastController *)broadcastController error:(nullable NSError *)error;

// broadcast controller delegate
- (void)broadcastController:(RPBroadcastController *_Nullable)broadcastController didFinishWithError:(NSError * __nullable)error;
- (void)broadcastController:(RPBroadcastController *_Nullable)broadcastController didUpdateServiceInfo:(NSDictionary <NSString *, NSObject <NSCoding> *> *_Nullable)serviceInfo;

@end

#endif
