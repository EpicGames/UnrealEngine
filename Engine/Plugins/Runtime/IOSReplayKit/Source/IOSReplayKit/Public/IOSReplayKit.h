// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogIOSReplayKit, Log, All);

/**
 * The public interface to this module.  In most cases, this interface is only public to sibling modules 
 * within this plugin.
 */
class IIOSReplayKitModuleInterface : public IModuleInterface
{

public:

	virtual void Initialize( bool bMicrophoneEnabled = false, bool bCameraEnabled = false ) = 0;
    
	virtual void StartRecording() = 0;
	virtual void StopRecording() = 0;
    
    virtual void StartCaptureToFile() = 0;
    virtual void StopCapture() = 0;
    
	virtual void StartBroadcast() = 0;
	virtual void PauseBroadcast() = 0;
	virtual void ResumeBroadcast() = 0;
	virtual void StopBroadcast() = 0;

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IIOSReplayKitModuleInterface& Get()
	{
		return FModuleManager::LoadModuleChecked< IIOSReplayKitModuleInterface >( "IOSReplayKit" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "IOSReplayKit" );
	}

    /** IModuleInterface implementation */
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
};


