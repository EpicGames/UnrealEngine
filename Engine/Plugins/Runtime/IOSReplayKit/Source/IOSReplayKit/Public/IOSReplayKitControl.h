// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "IOSReplayKitControl.generated.h"

UCLASS()
class IOSREPLAYKIT_API UIOSReplayKitControl : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	UFUNCTION( BlueprintCallable, Category="IOSReplayKit", meta = (WorldContext = "WorldContextObject", Keywords = "replaykit record"))
	static void StartRecording(bool bMicrophoneEnabled = true);

	UFUNCTION( BlueprintCallable, Category="IOSReplayKit", meta = (WorldContext = "WorldContextObject", Keywords = "replaykit record"))
	static void StopRecording();
    
    UFUNCTION( BlueprintCallable, Category="IOSReplayKit", meta = (WorldContext = "WorldContextObject", Keywords = "replaykit capture"))
    static void StartCaptureToFile(bool bMicrophoneEnabled = true);
    
    UFUNCTION( BlueprintCallable, Category="IOSReplayKit", meta = (WorldContext = "WorldContextObject", Keywords = "replaykit capture"))
    static void StopCapture();
};
